{
	"FileVersion": 3,
	"FriendlyName": "BackChannel",
	"Version": 1.0,
	"VersionName": "1",
	"Description": "BackChannel is an experimental plugin that allows external tools and apps to query for and push data into a running Unreal session.",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": true,

	"Modules": [
		{
			"Name": "BackChannel",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "Default",
			"PlatformDenyList": [
			]
		}
	]
}
