// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BackChannel : ModuleRules
{
	public BackChannel( ReadOnlyTargetRules Target ) : base( Target )
	{
		PrivateDependencyModuleNames.AddRange
			(
			new string[] {
				"Core",
				"Sockets",
				"Networking"
			}
		);
	}
}
