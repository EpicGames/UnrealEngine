// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackChannel/Protocol/OSC/BackChannelOSCBundle.h"
#include "BackChannelCommon.h"


const ANSICHAR* FBackChannelOSCBundle::BundleHeader = "#bundle";

FBackChannelOSCBundle::FBackChannelOSCBundle(OSCPacketMode InMode)
{
	Mode = InMode;
}

FBackChannelOSCBundle::~FBackChannelOSCBundle()
{
}

FBackChannelOSCBundle::FBackChannelOSCBundle(FBackChannelOSCBundle&& RHS)
{
	*this = MoveTemp(RHS);
}

FBackChannelOSCBundle& FBackChannelOSCBundle::operator=(FBackChannelOSCBundle&& RHS)
{
	Mode = RHS.Mode;
	TimeTag = RHS.TimeTag;

	return *this;
}

int32 FBackChannelOSCBundle::GetSize() const
{
	int32 Size = FCStringAnsi::Strlen(BundleHeader) + 1;
	Size += 8;	// timecode

	for (const auto& Element : Elements)
	{
		Size += 4;
		Size += Element.Num();
	}

	return Size;
}

void FBackChannelOSCBundle::AddElement(const void* InData, const int32 InSize)
{
	Elements.AddDefaulted(1);
	TArray<uint8>& NewElement = Elements.Last();

	NewElement.AddUninitialized(InSize);
	FMemory::Memcpy(NewElement.GetData(), InData, InSize);
}

int32 FBackChannelOSCBundle::GetElementCount() const
{
	return Elements.Num();
}

const TArray<uint8>& FBackChannelOSCBundle::GetElement(const int32 Index) const
{
	static TArray<uint8> Dummy;

	if (Index >= Elements.Num())
	{
		UE_LOGF(LogBackChannel, Error, "Index %d is greater than element count of %d", Index, GetElementCount());
		return Dummy;
	}

	return Elements[Index];
}

TArray<uint8> FBackChannelOSCBundle::WriteToBuffer() const
{
	TArray<uint8> Buffer;
	WriteToBuffer(Buffer);
	return Buffer;
}

void FBackChannelOSCBundle::WriteToBuffer(TArray<uint8>& OutBuffer) const
{
	const int32 Size = GetSize();
	const int32 BufferSize = OutBuffer.Num();

	// add enough space for our stuff
	OutBuffer.AddUninitialized(Size);

	// write after what was in the buffer
	uint8* DataPtr = (OutBuffer.GetData() + BufferSize);

	// write header (fixed size of 8-bytes)
	FCStringAnsi::Strncpy((ANSICHAR*)DataPtr, BundleHeader, Size);
	DataPtr[7] = 0;
	DataPtr += 8;

	// TODO - timecode
	DataPtr[0] = DataPtr[1] = 0;
	DataPtr += 8;

	// write all elements with a size header followed by the data
	for (const ElementData& Element : Elements)
	{
		int32 ElementSize = Element.Num();
		FMemory::Memcpy(DataPtr, &ElementSize, sizeof(ElementSize));
		DataPtr += 4;

		FMemory::Memcpy(DataPtr, Element.GetData(), ElementSize);
		DataPtr += ElementSize;
	}

	int32 BytesWritten = DataPtr - (OutBuffer.GetData() + BufferSize);
	check(BytesWritten == Size);
}

TSharedPtr<FBackChannelOSCBundle> FBackChannelOSCBundle::CreateFromBuffer(const void* Data, int32 DataLength)
{
	const uint8* pData = (const uint8*)Data;
	const uint8* pDataEnd = pData + DataLength;

	TSharedPtr<FBackChannelOSCBundle> NewBundle = MakeShareable(new FBackChannelOSCBundle(OSCPacketMode::Read));

	const ANSICHAR* Header = (const ANSICHAR*)Data;
	check(FCStringAnsi::Strcmp(Header, BundleHeader) == 0);

	// skip header
	pData += FCStringAnsi::Strlen(BundleHeader) + 1;

	// todo - timecode
	NewBundle->TimeTag = 0;
	pData += 8;

	while (pData < pDataEnd)
	{
		int32 ElementSize(0);
		FMemory::Memcpy(&ElementSize, pData, sizeof(ElementSize));
		pData += sizeof(ElementSize);

		NewBundle->AddElement(pData, ElementSize);

		pData += ElementSize;

		check(pData <= pDataEnd);
	}

	return NewBundle;
}
