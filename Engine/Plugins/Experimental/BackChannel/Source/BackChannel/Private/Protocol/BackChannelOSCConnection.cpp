// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackChannel/Protocol/OSC/BackChannelOSCConnection.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCMessage.h"
#include "BackChannelCommon.h"
#include "BackChannel/Transport/IBackChannelSocketConnection.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "Sockets.h"

FBackChannelOSCConnection::FBackChannelOSCConnection(TSharedRef<IBackChannelSocketConnection> InConnection)
	: Connection(InConnection)
{
	const int32 kDefaultBufferSize = 4096;
	ReceiveBuffer.AddUninitialized(kDefaultBufferSize);
}

FBackChannelOSCConnection::~FBackChannelOSCConnection()
{
	UE_LOGF(LogBackChannel, Verbose, "Destroying OSC Connection to %ls", *GetDescription());
	if (IsRunning)
	{
		Stop();
	}
}

FString FBackChannelOSCConnection::GetProtocolName() const
{
	return TEXT("BOSC");
}

TBackChannelSharedPtr<IBackChannelPacket> FBackChannelOSCConnection::CreatePacket()
{
	return MakeShared<FBackChannelOSCMessage, ESPMode::ThreadSafe>(OSCPacketMode::Write);
}

int FBackChannelOSCConnection::SendPacket(const TBackChannelSharedPtr<IBackChannelPacket>& Packet)
{
	FBackChannelOSCMessage* Message = static_cast<FBackChannelOSCMessage*>(Packet.Get());

	if (!SendPacket(*Message))
	{
		return -1;
	}

	return 0;
}


/* Bind a delegate to a message address */
FDelegateHandle FBackChannelOSCConnection::AddRouteDelegate(FStringView Path, FBackChannelRouteDelegate::FDelegate Delegate)
{
	FScopeLock Lock(&PacketMutex);
	return DispatchMap.AddRoute(Path, Delegate);
}

/* Remove a delegate handle */
void FBackChannelOSCConnection::RemoveRouteDelegate(FStringView Path, FDelegateHandle& InHandle)
{
	FScopeLock Lock(&PacketMutex);
	DispatchMap.RemoveRoute(Path, InHandle);
}

void FBackChannelOSCConnection::SetBufferSizes(int32 DesiredSendSize, int32 DesiredReceiveSize)
{
	if (Connection.IsValid())
	{
		Connection->SetBufferSizes(DesiredSendSize, DesiredReceiveSize);
	}
}


/*
FDelegateHandle FBackChannelOSCConnection::AddMessageHandler(const TCHAR* Path, FBackChannelDispatchDelegate::FDelegate Delegate)
{
	FScopeLock Lock(&PacketMutex);
	return LegacyDispatchMap.GetAddressHandler(Path).Add(Delegate);
}


void FBackChannelOSCConnection::RemoveMessageHandler(const TCHAR* Path, FDelegateHandle& InHandle)
{
	LegacyDispatchMap.GetAddressHandler(Path).Remove(InHandle);
	InHandle.Reset();
}*/

void FBackChannelOSCConnection::ReceiveAndDispatchMessages(const float MaxTime /*= 0*/)
{
	ReceiveMessages(MaxTime);
	DispatchMessages();
}

void FBackChannelOSCConnection::ReceiveMessages(const float MaxTime /*= 0*/)
{
	// Cap packets at 128MB.
	const int kMaxPacketSize = 128 * 1024 * 1024;

	const double StartTime = FPlatformTime::Seconds();
    
    //UE_LOGF(LogBackChannel, Log, "Starting Packet Check at %.02f", StartTime);

	bool KeepReceiving = false;

	int PacketsReceived = 0;

	do
	{
		FScopeLock Lock(&ReceiveMutex);

		Connection->GetSocket()->Wait(ESocketWaitConditions::WaitForRead, FTimespan(0, 0, MaxTime));

		int32 Received = Connection->ReceiveData(ReceiveBuffer.GetData() + ReceivedDataSize, ExpectedSizeOfNextPacket - ReceivedDataSize);

		if (Received > 0)
		{
			ReceivedDataSize += Received;

			if (ReceivedDataSize == ExpectedSizeOfNextPacket)
			{
				// reset this
				ReceivedDataSize = 0;

				if (ExpectedSizeOfNextPacket == 4)
				{
					int32 Size(0);
					FMemory::Memcpy(&Size, ReceiveBuffer.GetData(), sizeof(int32));

					if (Size >= 0 && Size <= kMaxPacketSize)
					{
						if (Size > ReceiveBuffer.Num())
						{
							ReceiveBuffer.AddUninitialized(Size - ReceiveBuffer.Num());
						}
						ExpectedSizeOfNextPacket = Size;
					}
					else
					{
						// if this is abnormally large it's likely a malformed packet so just reject it. We have to disconnect because we've no
                        // idea where the next valid packet in the stream is.
						UE_LOGF(LogBackChannel, Error,
						       "Received packet of %d bytes that was out of the range [0,%d]. Assuming data is malformed and disconnecring",
						       Size, kMaxPacketSize);
						HasErrorState = true;
						ExpectedSizeOfNextPacket = 4;
					}
				}
				else
				{
					// read packet
					TSharedPtr<FBackChannelOSCPacket> Packet = FBackChannelOSCPacket::CreateFromBuffer(ReceiveBuffer.GetData(), ExpectedSizeOfNextPacket);

					if (Packet.IsValid())
					{
                        FScopeLock PacketLock(&PacketMutex);
                        
						bool bAddPacket = true;

						if (Packet->GetType() == OSCPacketType::Message)
						{
							auto MsgPacket = StaticCastSharedPtr<FBackChannelOSCMessage>(Packet);

							const FString NewAddress = MsgPacket->GetPath();

							UE_LOGF(LogBackChannel, VeryVerbose, "Received message to %ls (tags:%ls, size:%d)", *NewAddress, *MsgPacket->GetTags(), ExpectedSizeOfNextPacket);

							int32 CurrentCount = GetMessageCountForPath(*NewAddress);

							int32 MaxMessages = GetMessageLimitForPath(*NewAddress);

							if (CurrentCount > 0)
							{
								UE_CLOGF(GBackChannelLogPackets, LogBackChannel, Log, "%ls has %d unprocessed messages", *NewAddress, CurrentCount + 1);

								if (MaxMessages > 0 && CurrentCount >= MaxMessages)
								{
									UE_LOGF(LogBackChannel, VeryVerbose, "Discarding old messages due to limit of %d", MaxMessages);
									RemoveMessagesWithPath(*NewAddress, 1);
								}
							}
						}
						else
						{
							UE_LOGF(LogBackChannel, VeryVerbose, "Received #bundle of %d bytes", ExpectedSizeOfNextPacket);
						}

						ReceivedPackets.Add(Packet);
					}

					ExpectedSizeOfNextPacket = 4;
					PacketsReceived++;
				}
			}
		}

		const double ElapsedTime = FPlatformTime::Seconds() - StartTime;

        // keep receiving until we run out of time, unless we received a packet
		KeepReceiving = ElapsedTime < MaxTime && PacketsReceived == 0 && HasErrorState == false;

	} while (KeepReceiving);
    
    const double ElapsedTime = FPlatformTime::Seconds() - StartTime;
    UE_LOGF(LogBackChannel, VeryVerbose, "Received %d packets in %.03f secs at %.03f", PacketsReceived, ElapsedTime, FPlatformTime::Seconds());
}

void FBackChannelOSCConnection::DispatchMessages()
{
	FScopeLock Lock(&PacketMutex);

	for (auto& Packet : ReceivedPackets)
	{
		if (Packet->GetType() == OSCPacketType::Message)
		{
			TSharedPtr<FBackChannelOSCMessage> Msg = StaticCastSharedPtr<FBackChannelOSCMessage>(Packet);

			UE_LOGF(LogBackChannel, VeryVerbose, "Dispatching %ls to handlers", *Msg->GetPath());

			if (!DispatchMap.DispatchMessage(*Msg.Get()))
			{
				UE_LOGF(LogBackChannel, Log, "Failed to displatch message to %ls. No handler?", *Msg->GetPath());
			}
			Msg->ResetRead();
		}
	}

	ReceivedPackets.Empty();
}

bool FBackChannelOSCConnection::StartReceiveThread()
{
	check(IsRunning == false);

	ExitRequested = false;

	// todo- expose this priority
	FRunnableThread* Thread = FRunnableThread::Create(this, TEXT("OSCHostConnection"), 1024 * 1024, TPri_AboveNormal);

	if (Thread)
	{
		IsRunning = true;
	}

	UE_LOGF(LogBackChannel, Verbose, "Started OSC Connection to %ls", *GetDescription());

	return Thread != nullptr;
}

/* Returns true if running in the background */
bool FBackChannelOSCConnection::IsThreaded() const
{
	return IsRunning;
}

uint32 FBackChannelOSCConnection::Run()
{
    const float kMaxReceiveTime = 1;
	const int32 kDefaultBufferSize = 4096;

	TArray<uint8> Buffer;
	Buffer.AddUninitialized(kDefaultBufferSize);

	UE_LOGF(LogBackChannel, Verbose, "OSC Connection to %ls is Running", *Connection->GetDescription());

	while (ExitRequested == false)
	{
        // This call itself will yield to the OS while waiting for recv so while this looks like a spinloop
        // it really isn't :)
        ReceiveAndDispatchMessages(kMaxReceiveTime);        
	}

	UE_LOGF(LogBackChannel, Verbose, "OSC Connection to %ls is exiting.", *Connection->GetDescription());
	IsRunning = false;
	return 0;
}

void FBackChannelOSCConnection::Stop()
{
	if (IsRunning)
	{
		UE_LOGF(LogBackChannel, Verbose, "Requesting OSC Connection to stop..");

		ExitRequested = true;

		while (IsRunning)
		{
			FPlatformProcess::SleepNoStats(0.01f);
		}
	}

	UE_LOGF(LogBackChannel, Verbose, "OSC Connection is stopped");
	Connection = nullptr;
}

bool FBackChannelOSCConnection::IsConnected() const
{
	return Connection->IsConnected() && (HasErrorState == false);
}

bool FBackChannelOSCConnection::SendPacket(FBackChannelOSCPacket& Packet)
{
	FBackChannelOSCMessage* MsgPacket = (FBackChannelOSCMessage*)&Packet;

	// todo : differentiate between UDP & TCP : for now we assume TCP which requires
	// the buffer size at the start of the array
	
	TArray<uint8> Data;

	// This int32 will hold the number of bytes added to the array by Packet.WriteToBuffer()
	Data.AddUninitialized(sizeof(int32));

	Packet.WriteToBuffer(Data);

	// Calculate the number of bytes added by the above call.
	const int32 BufferSize = Data.Num() - sizeof(int32);

	// Set the first 4 bytes to the buffer length
	FMemory::Memcpy(Data.GetData(), (void*)&BufferSize, sizeof(int32));

	UE_LOGF(LogBackChannel, VeryVerbose, "Sent message to %ls (tags:%ls, size:%d)", *MsgPacket->GetPath(), *MsgPacket->GetTags(), Data.Num());

	return SendPacketData(Data.GetData(), Data.Num());
}


bool FBackChannelOSCConnection::SendPacketData(const void* Data, const int32 DataLen)
{
	FScopeLock Lock(&SendMutex);

	if (!IsConnected())
	{
		return false;
	}

	int32 Sent = 0;

	if (DataLen > 0)
	{
		ANSICHAR* RawData = (ANSICHAR*)Data;
		check(FCStringAnsi::Strlen(RawData) < 64);
		Sent = Connection->SendData(Data, DataLen);
	}

	return Sent > 0;
}

FString FBackChannelOSCConnection::GetDescription()
{
	return FString::Printf(TEXT("OSCConnection to %s"), *Connection->GetDescription());
}

void FBackChannelOSCConnection::SetMessageOptions(const TCHAR* Path, int32 MaxQueuedMessages)
{
	FScopeLock Lock(&PacketMutex);
	MessageLimits.FindOrAdd(Path) = MaxQueuedMessages;
}

int32 FBackChannelOSCConnection::GetMessageCountForPath(const TCHAR* Path)
{
	FScopeLock Lock(&PacketMutex);
	
	int32 Count = 0;

	for (auto& Packet : ReceivedPackets)
	{
		if (Packet->GetType() == OSCPacketType::Message)
		{
			auto Msg = StaticCastSharedPtr<FBackChannelOSCMessage>(Packet);

			if (Msg->GetPath() == Path)
			{
				Count++;
			}
		}
	}

	return Count;
}


int32 FBackChannelOSCConnection::GetMessageLimitForPath(const TCHAR* InPath)
{
	FScopeLock Lock(&PacketMutex);

	FString Path = InPath;

	if (Path.EndsWith(TEXT("*")))
	{
		Path.LeftChopInline(1);
	}

	// todo - search for vest match, not first match
	for (auto KV : MessageLimits)
	{
		const FString& Key = KV.Key;
		if (Path.StartsWith(Key))
		{
			return KV.Value;
		}
	}

	return -1;
}

void FBackChannelOSCConnection::RemoveMessagesWithPath(const TCHAR* Path, const int32 Num /*= 0*/)
{
	FScopeLock Lock(&PacketMutex);

	auto It = ReceivedPackets.CreateIterator();

	int32 RemovedCount = 0;

	while (It)
	{
		auto Packet = *It;
		bool bRemove = false;

		if (Packet->GetType() == OSCPacketType::Message)
		{
			TSharedPtr<FBackChannelOSCMessage> Msg = StaticCastSharedPtr<FBackChannelOSCMessage>(Packet);

			if (Msg->GetPath() == Path)
			{
				bRemove = true;
			}
		}

		if (bRemove)
		{
			It.RemoveCurrent();
			RemovedCount++;

			if (Num > 0 && RemovedCount == Num)
			{
				break;
			}
		}
		else
		{
			It++;
		}
	}
}

