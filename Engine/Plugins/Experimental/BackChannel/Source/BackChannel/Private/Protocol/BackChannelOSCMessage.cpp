// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackChannel/Protocol/OSC/BackChannelOSCMessage.h"
#include "BackChannelCommon.h"

bool FBackChannelOSCMessage::bIsLegacyConnection = false;

FBackChannelOSCMessage::FBackChannelOSCMessage(OSCPacketMode InMode)
{
	Mode = InMode;
	TagIndex = 0;
	BufferIndex = 0;
}


FBackChannelOSCMessage::FBackChannelOSCMessage(const TCHAR* InAddress)
	: FBackChannelOSCMessage(OSCPacketMode::Write)
{
	SetPath(InAddress);
}

FBackChannelOSCMessage::~FBackChannelOSCMessage()
{
}


FBackChannelOSCMessage::FBackChannelOSCMessage(FBackChannelOSCMessage&& RHS)
{
	*this = MoveTemp(RHS);
}

FBackChannelOSCMessage& FBackChannelOSCMessage::operator=(FBackChannelOSCMessage&& RHS)
{
	Mode = RHS.Mode;
	Address = MoveTemp(RHS.Address);
	TagString = MoveTemp(RHS.TagString);
	TagIndex = RHS.TagIndex;
	Buffer = MoveTemp(RHS.Buffer);
	BufferIndex = RHS.BufferIndex;

	return *this;
}


int FBackChannelOSCMessage::SetPath(const TCHAR* InAddress)
{
	Address = InAddress;
	return 0;
}

void FBackChannelOSCMessage::ResetRead()
{
	check(IsReading());
	TagIndex = 0;
	BufferIndex = 0;
}

int FBackChannelOSCMessage::Read(const TCHAR* InName, FString& Value)
{
	check(IsReading());
	/*if (Mode == OSCPacketMode::Write)
	{
		// string should be null-terminated so allow space for that then
		// terminate
		int32 StartingIndex = BufferIndex;
		int32 StringLen = Value.Len();

		SerializeWrite(TEXT('s'), TCHAR_TO_ANSI(*Value), StringLen + 1);
		Buffer[StartingIndex + StringLen] = 0;
	}
	else*/
	{
		if (TagIndex == TagString.Len())
		{
			UE_LOGF(LogBackChannel, Warning, "OSCMessage(%ls): no more tags to read (expected 's', have %d tags). This can happen when the sender uses an older protocol version.", *Address, TagString.Len());
			return 1;
		}

		TCHAR CurrentTag = TagString[TagIndex];

		if (CurrentTag != 's')
		{
			UE_LOGF(LogBackChannel, Error, "OSCMessage: Requested tag 's' but next tag was %c", CurrentTag);
			return 1;
		}

		// get a pointer to the string (which is null-terminated) and read into the value
		ANSICHAR* pString = (ANSICHAR*)(Buffer.GetData() + BufferIndex);
		Value = ANSI_TO_TCHAR(pString);

		// manually adjust our buffer as if we read the string
		BufferIndex += RoundedArgumentSize(Value.Len() + 1);

		TagIndex++;
	}

	return 0;
}

int FBackChannelOSCMessage::Serialize(const TCHAR Code, void* InData, int32 InSize)
{
	if (Mode == OSCPacketMode::Read)
	{
		return ReadTagAndData(Code, InData, InSize);
	}
	else
	{
		return WriteTagAndData(Code, InData, InSize);
	}
}

int FBackChannelOSCMessage::WriteTag(const TCHAR Code)
{
	TagString += Code;
	TagIndex++;
	return 0; 
}

int FBackChannelOSCMessage::WriteTagAndData(const TCHAR Code, const void* InData, int32 InSize)
{
	WriteTag(Code);
	WriteData(InData, InSize);
	return 0;
}

int FBackChannelOSCMessage::WriteData(const void* InData, int32 InSize)
{
	// in OSC every write must be a multiple of 32-bits
	const int32 RoundedSize = RoundedArgumentSize(InSize);

	// allocate space and write.
	Buffer.AddUninitialized(RoundedSize);
	FMemory::Memcpy(Buffer.GetData() + BufferIndex, InData, InSize);
	BufferIndex += RoundedSize;

	return 0;
}

TCHAR FBackChannelOSCMessage::ReadTag(const TCHAR ExpectedTag, bool SuppressWarning /*= false*/)
{
	if (TagIndex == TagString.Len())
	{
		UE_CLOGF(!SuppressWarning, LogBackChannel, Error, "OSCMessage: Cannot read tag %c, no more tags!", ExpectedTag);
		return 0;
	}

	TCHAR CurrentTag = TagString[TagIndex];

	if (CurrentTag != ExpectedTag)
	{
		UE_CLOGF(!SuppressWarning, LogBackChannel, Error, "OSCMessage: Requested tag %c but next tag was %c", ExpectedTag, CurrentTag);
		return 0;
	}

	TagIndex++;
	return CurrentTag;
}

int FBackChannelOSCMessage::ReadTagAndData(const TCHAR ExpectedTag, void* InData, int32 InSize)
{
	TCHAR CurrentTag = ReadTag(ExpectedTag);

	if (CurrentTag == 0)
	{
		// readtag will generate errors
		return 1;
	}	
	ReadData(InData, InSize);
	return 0;
}

int FBackChannelOSCMessage::ReadData(void* InData, int32 InSize)
{
	FMemory::Memcpy(InData, Buffer.GetData() + BufferIndex, InSize);
	// in OSC every write must be a multiple of 32-bits
	BufferIndex += RoundedArgumentSize(InSize);
	return 0;
}

/* Read a blob of data from our arguments */
int FBackChannelOSCMessage::Read(const TCHAR* InName, void* OutBlob, int32 MaxBlobSize, int32& OutBlobSize)
{
	check(IsReading());

	bool ReadWasSuccess = false;

	if (ReadTag(TEXT('b')))
	{
		// take a copy here incase the user should use the same var for both args..
		int PassedMaxSize = MaxBlobSize;

		// read the blob size
		ReadData(&OutBlobSize, sizeof(int32));

		if (PassedMaxSize == 0)
		{
			// if MaxBlobSize was 0 then this was just a query so backout the tag and int that was read
			TagIndex--;
			BufferIndex -= sizeof(int32);
			ReadWasSuccess = true; // still success
		}
		else
		{
			if (ensure(OutBlob) && PassedMaxSize >= OutBlobSize)
			{
				ReadData(OutBlob, OutBlobSize);
			}
			else
			{
				// don't read any data, just skip it all so subsequent reads don't access duff data
				BufferIndex += RoundedArgumentSize(OutBlobSize);
				UE_LOGF(LogBackChannel, Error, "OSCMessage: buffer for reading blob was too small. Blob: %d bytes, buffer: %d", OutBlobSize, MaxBlobSize);
			}
		}

		ReadWasSuccess = (MaxBlobSize >= OutBlobSize || MaxBlobSize == 0);
	}

	return ReadWasSuccess ? 0 : 1;
}

void FBackChannelOSCMessage::GetComponentSizes(int32& OutAddressSize, int32& OutTagSize, int32& OutBufferSize) const
{
	OutAddressSize = RoundedArgumentSize(GetPath().Len()+1);

	const FString FinalTagString = FString::Printf(TEXT(",%s"), *GetTags());

	OutTagSize = RoundedArgumentSize(FinalTagString.Len()+1);		// we don't store the , internally

	OutBufferSize = BufferIndex;
}

int32 FBackChannelOSCMessage::GetSize() const
{
	int32 AddressSize, TagSize, BufferSize;
	GetComponentSizes(AddressSize, TagSize, BufferSize);

	return AddressSize + TagSize + BufferSize;
}

TArray<uint8> FBackChannelOSCMessage::WriteToBuffer() const
{
	TArray<uint8> OutBuffer;
	WriteToBuffer(OutBuffer);
	return OutBuffer;
}

void FBackChannelOSCMessage::WriteToBuffer(TArray<uint8>& OutBuffer) const
{
	int32 AddressSize(0);
	int32 TagSize(0);
	int32 BufferSize(0);
	GetComponentSizes(AddressSize, TagSize, BufferSize);

	const int32 kBufferOffset = OutBuffer.Num();
	const int32 kRequiredSize = AddressSize + TagSize + BufferSize;

	OutBuffer.AddUninitialized(kRequiredSize);

	ANSICHAR* pOutBuffer = (ANSICHAR*)OutBuffer.GetData() + kBufferOffset;
	
	const int32 kAddressLength = GetPath().Len()+1;
	const FString FinalTagString = FString::Printf(TEXT(",%s"), *GetTags());
	const int32 kTagLength = FinalTagString.Len() + 1;

	FCStringAnsi::Strncpy(pOutBuffer, TCHAR_TO_ANSI(*GetPath()), kAddressLength);
	pOutBuffer[kAddressLength] = 0;
	pOutBuffer += AddressSize;

	FCStringAnsi::Strncpy(pOutBuffer, TCHAR_TO_ANSI(*FinalTagString), kTagLength);
	pOutBuffer[kTagLength] = 0;
	pOutBuffer += TagSize;

	FMemory::Memcpy(pOutBuffer, Buffer.GetData(), BufferIndex);
}

TSharedPtr<FBackChannelOSCMessage> FBackChannelOSCMessage::CreateFromBuffer(const void * Data, int32 DataLength)
{
	const uint8* pParsedData = (const uint8*)Data;

	TSharedPtr<FBackChannelOSCMessage> NewMessage = MakeShareable(new FBackChannelOSCMessage(OSCPacketMode::Read));

	// first argument is the address as a null-terminated char*
	NewMessage->Address = ANSI_TO_TCHAR((ANSICHAR*)pParsedData);
	pParsedData += RoundedArgumentSize(NewMessage->Address.Len() + 1);

	// second argument is the address as a null-terminated char*
	FString TagString = ANSI_TO_TCHAR((ANSICHAR*)pParsedData);

	// remove the leading ,
	NewMessage->TagString = TagString.RightChop(TagString.Find(TEXT(","), ESearchCase::CaseSensitive)+1);

	pParsedData += RoundedArgumentSize(TagString.Len() + 1);

	// now pData points at the arguments.
	const int kArgLength = DataLength - (pParsedData - (const uint8*)Data);
	
	NewMessage->Buffer.AddUninitialized(kArgLength);
	FMemory::Memcpy(NewMessage->Buffer.GetData(), pParsedData, kArgLength);

	return NewMessage;
}

/*

FBackChannelOSCMessage& operator << (FBackChannelOSCMessage& Msg, int32& Value)
{
	Msg.Serialize(Value);
	return Msg;
}

FBackChannelOSCMessage& operator << (FBackChannelOSCMessage& Msg, float& Value)
{
	Msg.Serialize(Value);
	return Msg;
}

FBackChannelOSCMessage& operator << (FBackChannelOSCMessage& Msg, bool& Value)
{
	if (Msg.IsWriting())
	{
		int32 IntValue = Value ? 1 : 0;
		Msg.Serialize(IntValue);
	}
	else
	{
		int32 IntValue(0);
		Msg.Serialize(IntValue);
		Value = IntValue == 0 ? false : true;
	}
	return Msg;
}

FBackChannelOSCMessage& operator << (FBackChannelOSCMessage& Msg, TCHAR& Value)
{
	if (Msg.IsWriting())
	{
		int32 IntValue = Value;
		Msg.Serialize(IntValue);
	}
	else
	{
		int32 IntValue(0);
		Msg.Serialize(IntValue);
		Value = (TCHAR)IntValue;
	}
	return Msg;
}

FBackChannelOSCMessage& operator << (FBackChannelOSCMessage& Msg, FString& Value)
{
	Msg.Serialize(Value);
	return Msg;
}

*/
