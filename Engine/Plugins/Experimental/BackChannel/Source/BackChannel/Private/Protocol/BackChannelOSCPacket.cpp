// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackChannel/Protocol/OSC/BackChannelOSCPacket.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCBundle.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCMessage.h"



OSCPacketType FBackChannelOSCPacket::GetType(const void* Data, int32 DataLength)
{
	const ANSICHAR* CharData = (const ANSICHAR*)Data;

	if (CharData == nullptr)
	{
		return OSCPacketType::Invalid;
	}

	const ANSICHAR* BundleIdentifier = "#bundle";
	const ANSICHAR* MessageIdentifier = "";		// need to check whether "" is valid or it requires at least /

	const int32 MessageIdentifierLen = FCStringAnsi::Strlen(MessageIdentifier);

	if (DataLength >= FCStringAnsi::Strlen(BundleIdentifier))
	{
		if (FCStringAnsi::Stricmp(CharData, BundleIdentifier) == 0)
		{
			return OSCPacketType::Bundle;
		}
	}

	if (DataLength >= MessageIdentifierLen)
	{
		if (FCStringAnsi::Strnicmp(CharData, MessageIdentifier, MessageIdentifierLen) == 0)
		{
			return OSCPacketType::Message;
		}
	}

	return OSCPacketType::Invalid;
}

TSharedPtr<FBackChannelOSCPacket> FBackChannelOSCPacket::CreateFromBuffer(const void * Data, int32 DataLength)
{
	OSCPacketType Type = GetType(Data, DataLength);

	if (Type == OSCPacketType::Bundle)
	{
		return FBackChannelOSCBundle::CreateFromBuffer(Data, DataLength);
	}
	else if (Type == OSCPacketType::Message)
	{
		return FBackChannelOSCMessage::CreateFromBuffer(Data, DataLength);
	}

	return nullptr;
}
