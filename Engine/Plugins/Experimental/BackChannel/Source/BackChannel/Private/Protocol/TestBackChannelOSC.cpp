// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackChannel/Protocol/OSC/BackChannelOSCBundle.h"
#include "Misc/AutomationTest.h"
#include "BackChannel/Protocol/OSC/BackChannelOSCMessage.h"


#if WITH_DEV_AUTOMATION_TESTS

class FBackChannelTestOSCBase : public FAutomationTestBase
{

public:

	FBackChannelTestOSCBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask) {}

};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FBackChannelTestOSCMessage, FBackChannelTestOSCBase, "BackChannel.TestOSCMessage", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FBackChannelTestOSCBundle, FBackChannelTestOSCBase, "BackChannel.TestOSCBundle", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FBackChannelTestOSCBundleWithMessages, FBackChannelTestOSCBase, "BackChannel.TestOSCBundleWithMessages", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)


bool FBackChannelTestOSCMessage::RunTest(const FString& Parameters)
{
	FBackChannelOSCMessage Message(OSCPacketMode::Write);
	
	Message.SetPath(TEXT("/foo"));
	
	int32 IntValue = 1;
	float FloatValue = 2.5;
	FString StringValue = TEXT("Hello!");
	TArray<uint8> AnswerArray;

	const int kArraySize = 33;
	const int kArrayValue = 42;

	for (int i = 0; i < kArraySize; i++)
	{
		AnswerArray.Add(kArrayValue);
	}

	// #agrant todo - cleanup
	// Message << IntValue << FloatValue << StringValue << AnswerArray;
	Message.Write(TEXT("Int"), IntValue);
	Message.Write(TEXT("true"), true);
	Message.Write(TEXT("false"), false);
	Message.Write(TEXT("Float"), FloatValue);
	Message.Write(TEXT("String"), StringValue);
	Message.Write(TEXT("Array"), AnswerArray);

	FString Address = Message.GetPath();
	FString Tags = Message.GetTags();
	const int32 ArgSize = Message.GetArgumentSize();

	const int RoundedStringSize = FBackChannelOSCMessage::RoundedArgumentSize(StringValue.Len() + 1);
	const int RoundedArraySize = FBackChannelOSCMessage::RoundedArgumentSize(kArraySize);

	const int ExpectedArgSize = 4 + 4 + RoundedStringSize + 4 + RoundedArraySize;
	const int ExpectedBufferSize = ExpectedArgSize + FBackChannelOSCMessage::RoundedArgumentSize(Address.Len() + 1) + FBackChannelOSCMessage::RoundedArgumentSize(Tags.Len() + 1);

	// verify this address and tags...
	check(Address == TEXT("/foo"));
	check(Tags == TEXT("iTFfsb"));
	check(ArgSize == ExpectedArgSize);

	TArray<uint8> Buffer;

	Message.WriteToBuffer(Buffer);

	check(Buffer.Num() == ExpectedBufferSize);
	check(FBackChannelOSCPacket::GetType(Buffer.GetData(), Buffer.Num()) == OSCPacketType::Message);

	TSharedPtr<FBackChannelOSCMessage> NewMessage = FBackChannelOSCMessage::CreateFromBuffer(Buffer.GetData(), Buffer.Num());

	// read them back
	int32 OutIntValue(0);
	float OutFloatValue(0);
	bool OutTrueValue(false);
	bool OutFalseValue(true);
	FString OutStringValue;
	TArray<uint8> OutArray;

	NewMessage->Read(TEXT("Int"), OutIntValue);
	NewMessage->Read(TEXT("true"), OutTrueValue);
	NewMessage->Read(TEXT("false"), OutFalseValue);
	NewMessage->Read(TEXT("Float"), OutFloatValue);
	NewMessage->Read(TEXT("String"), OutStringValue);
	NewMessage->Read(TEXT("Array"), OutArray);

	check(OutIntValue == IntValue);
	check(OutFloatValue == OutFloatValue);
	check(OutStringValue == OutStringValue);
	check(OutTrueValue == true);
	check(OutFalseValue == false);
	check(OutArray.Num() == AnswerArray.Num());
	
	for (int i = 0; i < OutArray.Num(); i++)
	{
		check(OutArray[i] == AnswerArray[i]);
	}

	return true;
}


bool FBackChannelTestOSCBundle::RunTest(const FString& Parameters)
{
	TSharedPtr<FBackChannelOSCBundle> Bundle = MakeShareable(new FBackChannelOSCBundle(OSCPacketMode::Write));

	FString TestString1 = TEXT("Hello World!");
	FString TestString2 = TEXT("Hello World Again!");

	Bundle->AddElement(*TestString1, (TestString1.Len() + 1) * sizeof(TCHAR));
	Bundle->AddElement(*TestString2, (TestString2.Len() + 1) * sizeof(TCHAR));

	// first loop tests the bundle we just constructed, second loop tests it
	// after serializing to and from a buffer
	for (int i = 0; i < 2; i++)
	{
		check(Bundle->GetElementCount() == 2);

		TArray<uint8> Data1, Data2;
		Data1 = Bundle->GetElement(0);
		Data2 = Bundle->GetElement(1);

		const TCHAR* pString1 = (const TCHAR*)Data1.GetData();
		const TCHAR* pString2 = (const TCHAR*)Data2.GetData();

		check(TestString1 == pString1);
		check(TestString2 == pString2);

		TArray<uint8> BundleData;
		
		// write to the buffer
		Bundle->WriteToBuffer(BundleData);

		TSharedPtr<FBackChannelOSCPacket> Packet = FBackChannelOSCPacket::CreateFromBuffer(BundleData.GetData(), BundleData.Num());

		check(Packet->GetType() == OSCPacketType::Bundle);

		Bundle = StaticCastSharedPtr<FBackChannelOSCBundle>(Packet);
	}

	return true;
}

bool FBackChannelTestOSCBundleWithMessages::RunTest(const FString& Parameters)
{
	TSharedPtr<FBackChannelOSCMessage> Msg1 = MakeShareable(new FBackChannelOSCMessage(OSCPacketMode::Write));
	TSharedPtr<FBackChannelOSCMessage> Msg2 = MakeShareable(new FBackChannelOSCMessage(OSCPacketMode::Write));

	FString Msg1Test = TEXT("This is Message 1");
	FString Msg2Test = TEXT("This is Message 2");

	// #agrant cleanuo
	//(*Msg1) << Msg1Test;
	//*Msg2 << Msg2Test;

	Msg1->Write(TEXT("String"), *Msg1Test);
	Msg2->Write(TEXT("String"), *Msg2Test);

	TSharedPtr<FBackChannelOSCBundle> Bundle = MakeShareable(new FBackChannelOSCBundle(OSCPacketMode::Write));

	TArray<uint8> Msg1Buffer, Msg2Buffer;
	Msg1->WriteToBuffer(Msg1Buffer);
	Msg2->WriteToBuffer(Msg2Buffer);

	Bundle->AddElement(Msg1Buffer.GetData(), Msg1Buffer.Num());
	Bundle->AddElement(Msg2Buffer.GetData(), Msg2Buffer.Num());

	TArray<uint8> BundleBuffer;
	Bundle->WriteToBuffer(BundleBuffer);

	TSharedPtr<FBackChannelOSCPacket> RecreatedPacket = FBackChannelOSCPacket::CreateFromBuffer(BundleBuffer.GetData(), BundleBuffer.Num());

	check(RecreatedPacket->GetType() == OSCPacketType::Bundle);

	TSharedPtr<FBackChannelOSCBundle> RecreatedBuffer = StaticCastSharedPtr<FBackChannelOSCBundle>(RecreatedPacket);

	check(RecreatedBuffer->GetElementCount() == 2);

	const TArray<uint8>& Element1 = RecreatedBuffer->GetElement(0);
	const TArray<uint8>& Element2 = RecreatedBuffer->GetElement(1);

	check(FBackChannelOSCPacket::GetType(Element1.GetData(), Element1.Num()) == OSCPacketType::Message);
	check(FBackChannelOSCPacket::GetType(Element2.GetData(), Element2.Num()) == OSCPacketType::Message);

	TSharedPtr<FBackChannelOSCMessage> RecreatedMsg1 = FBackChannelOSCMessage::CreateFromBuffer(Element1.GetData(), Element1.Num());
	TSharedPtr<FBackChannelOSCMessage> RecreatedMsg2 = FBackChannelOSCMessage::CreateFromBuffer(Element2.GetData(), Element2.Num());

	check(RecreatedMsg1.IsValid());
	check(RecreatedMsg2.IsValid());

	FString OutMsg1Text, OutMsg2Text;

	// #agrant cleanup
	//*RecreatedMsg1 << OutMsg1Text;
	//*RecreatedMsg2 << OutMsg2Text;
	RecreatedMsg1->Read(TEXT("String"), OutMsg1Text);
	RecreatedMsg2->Read(TEXT("String"), OutMsg2Text);

	check(OutMsg1Text == Msg1Test);
	check(OutMsg2Text == Msg2Test);

	return true;
}



#endif // WITH_DEV_AUTOMATION_TESTS
