// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/BackChannelConnection.h"
#include "BackChannelCommon.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "IPAddress.h"
#include "SocketSubsystem.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Sockets.h"
#include "Stats/Stats.h"
#include "Stats/StatsTrace.h"

DECLARE_STATS_GROUP(TEXT("BackChannel"), STATGROUP_BackChannel, STATCAT_Advanced);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("BC.PacketsSent/s"), STAT_BackChannelPacketsSent, STATGROUP_BackChannel);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("BC.PacketsRecv/s"), STAT_BackChannelPacketsRecv, STATGROUP_BackChannel);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("BC.BytesSent/s"), STAT_BackChannelBytesSent, STATGROUP_BackChannel);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("BC.BytesRecv/s"), STAT_BackChannelBytesRecv, STATGROUP_BackChannel);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("BC.Errors/s"), STAT_BackChannelErrors, STATGROUP_BackChannel);


int32 FBackChannelConnection::SendBufferSize = 2 * 1024 * 1024;
int32 FBackChannelConnection::ReceiveBufferSize = 2 * 1024 * 1024;

int32 GBackChannelLogPackets = 0;
static FAutoConsoleVariableRef BCCVarLogPackets(
	TEXT("backchannel.logpackets"), GBackChannelLogPackets,
	TEXT("Logs incoming packets"),
	ECVF_Default);

int32 GBackChannelLogErrors = 1;
static FAutoConsoleVariableRef BCCVarLogErrors(
	TEXT("backchannel.logerrors"), GBackChannelLogErrors,
	TEXT("Logs packet errors"),
	ECVF_Default);

FBackChannelConnection::FBackChannelConnection()
{
	Socket = nullptr;
	IsListener = false;
	TimeSinceStatsSet = 0;
	// Allow the app to override
	GConfig->GetInt(TEXT("BackChannel"), TEXT("SendBufferSize"), SendBufferSize, GEngineIni);
	GConfig->GetInt(TEXT("BackChannel"), TEXT("RecvBufferSize"), ReceiveBufferSize, GEngineIni);
}

FBackChannelConnection::~FBackChannelConnection()
{
	if (Socket)
	{
		Close();
	}
}

/* Todo - Proper stats */
uint32	FBackChannelConnection::GetPacketsReceived() const
{
	return ConnectionStats.PacketsReceived;
}

bool FBackChannelConnection::IsConnected() const
{
	FBackChannelConnection* NonConstThis = const_cast<FBackChannelConnection*>(this);
	FScopeLock Lock(&NonConstThis->SocketMutex);
	return Socket != nullptr && Socket->GetConnectionState() == ESocketConnectionState::SCS_Connected;
}

bool FBackChannelConnection::IsListening() const
{
	return IsListener;
}

FString	FBackChannelConnection::GetDescription() const
{
	FBackChannelConnection* NonConstThis = const_cast<FBackChannelConnection*>(this);
	FScopeLock Lock(&NonConstThis->SocketMutex);

	return Socket ? Socket->GetDescription() : TEXT("No Socket");
}

void FBackChannelConnection::Close()
{
	FScopeLock Lock(&SocketMutex);
	if (Socket)
	{
		UE_LOGF(LogBackChannel, Log, "Closing connection %ls", *Socket->GetDescription());
		Socket->Close();
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
		Socket = nullptr;
	}
}

void FBackChannelConnection::CloseWithError(const TCHAR* Error, FSocket* InSocket)
{
	const TCHAR* SocketErr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetSocketError(SE_GET_LAST_ERROR_CODE);

	if (InSocket == nullptr)
	{
		InSocket = Socket;
	}

	FString SockDesc = InSocket != nullptr ? InSocket->GetDescription() : TEXT("(No Socket)");

	UE_LOGF(LogBackChannel, Error, "%ls, Err: %ls, Socket:%ls", Error, SocketErr, *SockDesc);

	Close();
}

bool FBackChannelConnection::Connect(const TCHAR* InEndPoint)
{
	FScopeLock Lock(&SocketMutex);

	if (IsConnected())
	{
		Close();
	}

	IsAttemptingConnection = true;

	FString LocalEndPoint = InEndPoint;
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);

	// Parse the endpoint — try IPv4 first (fast path), then fall back to generic parsing for IPv6
	TSharedPtr<FInternetAddr> TargetAddr;
	FIPv4Endpoint EndPointv4;
	if (FIPv4Endpoint::Parse(LocalEndPoint, EndPointv4))
	{
		TargetAddr = EndPointv4.ToInternetAddr();
	}
	else
	{
		// Try generic parsing for IPv6, e.g. "[::1]:2049"
		FString AddressPart;
		int32 Port = 0;
		if (LocalEndPoint.StartsWith(TEXT("[")))
		{
			// Format: [IPv6Address]:Port
			int32 CloseBracket;
			if (LocalEndPoint.FindChar(TEXT(']'), CloseBracket))
			{
				AddressPart = LocalEndPoint.Mid(1, CloseBracket - 1);
				FString PortStr = LocalEndPoint.Mid(CloseBracket + 1);
				if (PortStr.StartsWith(TEXT(":")))
				{
					LexFromString(Port, *PortStr.Mid(1));
				}
			}
		}

		if (!AddressPart.IsEmpty())
		{
			if (Port <= 0 || Port > 65535)
			{
				UE_LOGF(LogBackChannel, Error, "Invalid or missing port in endpoint '%ls'", InEndPoint);
				return false;
			}

			TargetAddr = SocketSubsystem->GetAddressFromString(AddressPart);
			if (TargetAddr.IsValid())
			{
				TargetAddr->SetPort(Port);
			}
		}
	}

	if (!TargetAddr.IsValid())
	{
		UE_LOGF(LogBackChannel, Error, "Failed to parse endpoint '%ls'", InEndPoint);
		return false;
	}

	// Create socket with the correct protocol family for the target address
	FName ProtocolType = TargetAddr->GetProtocolType();
	FSocket* NewSocket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("FBackChannelConnection Client Socket"), ProtocolType);
	if (NewSocket)
	{
		NewSocket->SetNonBlocking();

		// set buffer sizes
		SetSocketBufferSizes(NewSocket, SendBufferSize, ReceiveBufferSize);

		bool Success = NewSocket->Connect(*TargetAddr);

		if (!Success)
		{
			ESocketErrors LastErr = SocketSubsystem->GetLastErrorCode();

			if (LastErr == SE_EINPROGRESS || LastErr == SE_EWOULDBLOCK)
			{
				Success = true;
			}
			else
			{
				UE_LOGF(LogBackChannel, Warning, "Connect failed with error code (%d) error (%ls)", LastErr, SocketSubsystem->GetSocketError(LastErr));
			}
		}

		if (Success)
		{
			UE_LOGF(LogBackChannel, Log, "Opening connection to %ls (localport: %d)", *NewSocket->GetDescription(), NewSocket->GetPortNo());
			Attach(NewSocket);
		}
		else
		{
			CloseWithError(*FString::Printf(TEXT("Failed to open connection to %s."), InEndPoint), NewSocket);
			SocketSubsystem->DestroySocket(NewSocket);
		}
	}

	return Socket != nullptr;
}

// public version that is applied to our socket
void FBackChannelConnection::SetBufferSizes(int32 DesiredSendSize, int32 DesiredReceiveSize)
{
	// Save these for any future connections
	SendBufferSize = FMath::Max(DesiredSendSize, SendBufferSize);
	ReceiveBufferSize = FMath::Max(DesiredReceiveSize, ReceiveBufferSize);

	// If we have a socket, apply it now
	if (Socket != nullptr)
	{
		SetSocketBufferSizes(Socket, SendBufferSize, SendBufferSize);
	}
}

void FBackChannelConnection::SetSocketBufferSizes(FSocket* NewSocket, int32 DesiredSendSize, int32 DesiredReceiveSize)
{
	int32 AllocatedSendSize = 0;
	int32 AllocatedReceiveSize = 0;

	int32 RequestedSendSize = DesiredSendSize;
	int32 RequestedReceiveSize = DesiredReceiveSize;
	bool bWasSet = false;
		
	// Send Buffer
	while (AllocatedSendSize < RequestedSendSize && !bWasSet)
	{
		// note - it's possible for AllocatedSize to change and bWasSet = false if the socket has already had a buffer
		// size set that's the mac supported.
		bWasSet = NewSocket->SetSendBufferSize(RequestedSendSize, AllocatedSendSize);

		// If we didn't get what we want assume failure could mean
		// no change (unsupported size), an OS default, or an OS max so
		// try again if its less than 50% of what we asked for
		if (!bWasSet && AllocatedSendSize < RequestedSendSize)
		{
			RequestedSendSize = RequestedSendSize / 2;
		}
	}
	
	if (AllocatedSendSize != DesiredSendSize)
	{
		UE_LOGF(LogBackChannel, Warning, "Wanted send buffer of %d for %ls but OS only allowed %d", DesiredSendSize, *NewSocket->GetDescription(), AllocatedSendSize);
	}
	else
	{
		UE_LOGF(LogBackChannel, Log, "Set send buffer to %d bytes for %ls", AllocatedSendSize, *NewSocket->GetDescription());
	}
	
	bWasSet = false;

	// Set Receive buffer
	while (AllocatedReceiveSize < RequestedReceiveSize && !bWasSet)
	{
		bWasSet = NewSocket->SetReceiveBufferSize(RequestedReceiveSize, AllocatedReceiveSize);
		
		// If we didn't get what we want assume failure could mean
		// no change (unsupported size), an OS default, or an OS max so
		// try again if its less than 50% of what we asked for
		if (!bWasSet && AllocatedReceiveSize < RequestedReceiveSize)
		{
			RequestedReceiveSize = RequestedReceiveSize / 2;
		}
	}
	
	if (AllocatedReceiveSize != DesiredReceiveSize)
	{
		UE_LOGF(LogBackChannel, Warning, "Wanted receive buffer of %d for %ls but OS only allowed %d", DesiredReceiveSize, *NewSocket->GetDescription(), AllocatedReceiveSize);
	}
	else
	{
		UE_LOGF(LogBackChannel, Log, "Set receive buffer to %d bytes for %ls", AllocatedSendSize, *NewSocket->GetDescription());
	}
}

bool FBackChannelConnection::Listen(const int16 Port)
{
	FScopeLock Lock(&SocketMutex);

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);

	// Try IPv6 dual-stack first (accepts both IPv4 and IPv6 connections)
	FSocket* NewSocket = nullptr;
	TSharedRef<FInternetAddr> BindAddr = SocketSubsystem->CreateInternetAddr();

#if PLATFORM_HAS_BSD_IPV6_SOCKETS
	NewSocket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("FBackChannelConnection Listener"), FNetworkProtocolTypes::IPv6);
	if (NewSocket)
	{
		// Platform socket subsystems already set IPV6_V6ONLY=false in CreateSocket(),
		// enabling dual-stack: this IPv6 socket will accept IPv4 connections too.

		// Bind to IPv6 any-address (::)
		BindAddr = SocketSubsystem->CreateInternetAddr(FNetworkProtocolTypes::IPv6);
		BindAddr->SetAnyAddress();
		BindAddr->SetPort(Port);
	}
#endif

	if (!NewSocket)
	{
		// Fall back to IPv4-only
		NewSocket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("FBackChannelConnection Listener"), FNetworkProtocolTypes::IPv4);
		if (NewSocket)
		{
			FIPv4Endpoint Endpoint(FIPv4Address::Any, Port);
			BindAddr = Endpoint.ToInternetAddr();
		}
	}

	if (NewSocket != nullptr)
	{
		bool Error = !NewSocket->SetReuseAddr(true);

		if (!Error)
		{
			Error = !NewSocket->SetRecvErr();
		}

		if (!Error)
		{
			Error = !NewSocket->SetNonBlocking();
		}

		if (!Error)
		{
			Error = !NewSocket->Bind(*BindAddr);
		}

		if (!Error)
		{
			Error = !NewSocket->Listen(8);
		}

		if (Error)
		{
			const TCHAR* SocketErr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetSocketError(SE_GET_LAST_ERROR_CODE);
			GLog->Logf(TEXT("Failed to create the listen socket as configured. %s"), SocketErr);

			SocketSubsystem->DestroySocket(NewSocket);

			NewSocket = nullptr;
		}
	}

	if (NewSocket == nullptr)
	{
		const TCHAR* SocketErr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetSocketError(SE_GET_LAST_ERROR_CODE);

		UE_LOGF(LogBackChannel, Error, "Failed to open socket on port %d. Err: %ls", Port, SocketErr);
		CloseWithError(*FString::Printf(TEXT("Failed to start listening on port %d"), Port));
	}
	else
	{
		UE_LOGF(LogBackChannel, Log, "Listening on %ls (localport: %d)", *NewSocket->GetDescription(), NewSocket->GetPortNo());
		Attach(NewSocket);
		IsListener = true;
	}

	return NewSocket != nullptr;
}

bool FBackChannelConnection::WaitForConnection(double InTimeout, TFunction<bool(TSharedRef<IBackChannelSocketConnection>)> InDelegate)
{
	FScopeLock Lock(&SocketMutex);

	if (!Socket)
	{
		UE_LOGF(LogBackChannel, Error, "Connection has no socket. Call Listen/Connect before WaitForConnection");
		return false;
	}

	FTimespan SleepTime = FTimespan(0, 0, InTimeout);

	// handle incoming connections

	bool CheckSucceeded = false;
	bool HasConnection = false;

	if (IsListener)
	{
		CheckSucceeded = Socket->WaitForPendingConnection(HasConnection, SleepTime);
	}
	else
	{
		ESocketConnectionState State = Socket->GetConnectionState();

		if (State == ESocketConnectionState::SCS_ConnectionError)
		{
			ESocketErrors Err = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLastErrorCode();

			if (Err == ESocketErrors::SE_NO_ERROR)
			{
				CheckSucceeded = true;
			}
			else
			{
				const TCHAR* SocketErr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetSocketError(Err);
				UE_LOGF(LogBackChannel, Warning, "Socket has error %ls", SocketErr);
			}
		}
		else
		{
			CheckSucceeded = true;
			HasConnection = Socket->Wait(ESocketWaitConditions::WaitForWrite, SleepTime);
		}
	}

	if (CheckSucceeded)
	{
		if (HasConnection)
		{
			UE_LOGF(LogBackChannel, Log, "Found connection on %ls", *Socket->GetDescription());

			if (IsListener == false)
			{
				InDelegate(AsShared());
			}
			else
			{
				TSharedRef<FInternetAddr> RemoteAddress = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();

				FSocket* ConnectionSocket = Socket->Accept(*RemoteAddress, TEXT("RemoteConnection"));

				if (ConnectionSocket != nullptr)
				{
					// Each platform can inherit different socket options from the listen socket so set ours again
					{
						ConnectionSocket->SetNonBlocking();

						// set buffer sizes
						SetSocketBufferSizes(ConnectionSocket, SendBufferSize, ReceiveBufferSize);
					}

					TSharedRef<FBackChannelConnection> BCConnection = MakeShareable(new FBackChannelConnection);
					BCConnection->Attach(ConnectionSocket);

					if (InDelegate(BCConnection) == false)
					{
						UE_LOGF(LogBackChannel, Warning, "Calling code rejected connection on %ls", *Socket->GetDescription());
						BCConnection->Close();
					}
					else
					{
						UE_LOGF(LogBackChannel, Log, "Accepted connection from %ls on %ls", *RemoteAddress->ToString(true),  *Socket->GetDescription());
					}
				}
			}
		}
	}
	else
	{
		CloseWithError(TEXT("Connection Check Failed"));
	}

	return CheckSucceeded;
}

bool FBackChannelConnection::Attach(FSocket* InSocket)
{
	FScopeLock Lock(&SocketMutex);

	check(Socket == nullptr);

	Socket = InSocket;
	return true;
}


int32 FBackChannelConnection::SendData(const void* InData, const int32 InSize)
{
	FScopeLock Lock(&SocketMutex);
	if (!Socket)
	{
		return -1;
	}

	int32 BytesSent(0);
	Socket->Send((const uint8*)InData, InSize, BytesSent);

	ResetStatsIfTime();

	if (BytesSent == -1)
	{
		if (GBackChannelLogErrors)
		{
			ESocketErrors LastError = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLastErrorCode();
			const TCHAR* SocketErr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetSocketError(LastError);
			UE_CLOGF(GBackChannelLogErrors, LogBackChannel, Error, "Failed to send %d bytes of data to %ls. Err: %ls", InSize, *GetDescription(), SocketErr);
		}

		ConnectionStats.Errors++;
		ConnectionStats.LastErrorTime = FPlatformTime::Seconds();
	}
	else
	{
		ConnectionStats.PacketsSent++;
		ConnectionStats.BytesSent += BytesSent;
		ConnectionStats.LastSuccessTime = FPlatformTime::Seconds();
		ConnectionStats.LastSendTime = ConnectionStats.LastSuccessTime;

		UE_CLOGF(GBackChannelLogPackets, LogBackChannel, Log, "Sent %d bytes of data", BytesSent);
	}
	return BytesSent;
}

int32 FBackChannelConnection::ReceiveData(void* OutBuffer, const int32 BufferSize)
{
	FScopeLock Lock(&SocketMutex);
	if (!Socket)
	{
		return 0;
	}

	int32 BytesRead(0);
	Socket->Recv((uint8*)OutBuffer, BufferSize, BytesRead, ESocketReceiveFlags::None);	

	ResetStatsIfTime();

	// todo - close connection on certain errors
	if (BytesRead > 0)
	{
		ConnectionStats.PacketsReceived++;
		ConnectionStats.BytesReceived += BytesRead;
		ConnectionStats.LastSuccessTime = FPlatformTime::Seconds();
		ConnectionStats.LastReceiveTime = ConnectionStats.LastSuccessTime;
		UE_CLOGF(GBackChannelLogPackets, LogBackChannel, Log, "Received %d bytes of data", BytesRead);
	}
	else if (BytesRead < 0)
	{
		// note - FSocket consumes WOULDBLOCk errors so there's not a lot to do here..
	}

	return BytesRead;
}

void FBackChannelConnection::ResetStatsIfTime()
{
	const double TimeNow = FPlatformTime::Seconds();

	if (TimeNow - TimeSinceStatsSet  >= 1.0)
	{
		// stats reflect the last second
		int PacketsReceived = ConnectionStats.PacketsReceived - LastStats.PacketsReceived;
		int BytesReceived = ConnectionStats.BytesReceived - LastStats.BytesReceived;
		int PacketsSent = ConnectionStats.PacketsSent - LastStats.PacketsSent;
		int BytesSent = ConnectionStats.BytesSent - LastStats.BytesSent;
		int Errors = ConnectionStats.Errors - LastStats.Errors;

		SET_DWORD_STAT(STAT_BackChannelPacketsRecv, PacketsReceived);
		SET_DWORD_STAT(STAT_BackChannelBytesRecv, BytesReceived);
		SET_DWORD_STAT(STAT_BackChannelPacketsSent, PacketsSent);
		SET_DWORD_STAT(STAT_BackChannelBytesSent, BytesSent);
		SET_DWORD_STAT(STAT_BackChannelErrors, Errors);

		LastStats = ConnectionStats;
	}
}
