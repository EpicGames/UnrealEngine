// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackChannel/Transport/IBackChannelSocketConnection.h"
#include "HAL/ThreadSafeBool.h"

#define UE_API BACKCHANNEL_API

class FSocket;

/**
* BackChannelClient implementation.
*
*/
class FBackChannelConnection : public IBackChannelSocketConnection, public TSharedFromThis<FBackChannelConnection>
{
public:

	UE_API FBackChannelConnection();
	UE_API ~FBackChannelConnection();

	/* Start connecting to the specified port for incoming connections. Use WaitForConnection to check status. */
	UE_API virtual bool Connect(const TCHAR* InEndPoint) override;

	/* Start listening on the specified port for incoming connections. Use WaitForConnection to accept one. */
	UE_API virtual bool Listen(const int16 Port) override;

	/* Close the connection */
	UE_API virtual void Close() override;

	/* Waits for an icoming or outgoing connection to be made */
	UE_API virtual bool WaitForConnection(double InTimeout, TFunction<bool(TSharedRef<IBackChannelSocketConnection>)> InDelegate) override;

	/* Attach this connection to the provided socket */
	UE_API bool Attach(FSocket* InSocket);

	/* Send data over our connection. The number of bytes sent is returned */
	UE_API virtual int32 SendData(const void* InData, const int32 InSize) override;

	/* Read data from our remote connection. The number of bytes received is returned */
	UE_API virtual int32 ReceiveData(void* OutBuffer, const int32 BufferSize) override;

	/* Return our current connection state */
	UE_API virtual bool IsConnected() const override;

	/* Returns true if this connection is currently listening for incoming connections */
	UE_API virtual bool IsListening() const override;

	/* Return a string describing this connection */
	UE_API virtual FString	GetDescription() const override;

	/* Return the underlying socket (if any) for this connection */
	virtual FSocket* GetSocket() override { return Socket; }

	/* Todo - Proper stats */
	UE_API virtual uint32	GetPacketsReceived() const override;

	/* Set the specified send and receive buffer sizes, if supported */
	UE_API virtual void SetBufferSizes(int32 DesiredSendSize, int32 DesiredReceiveSize) override;

	const FConnectionStats& GetConnectionStats() const override { return ConnectionStats; }

private:
	static UE_API int32 SendBufferSize;
	static UE_API int32 ReceiveBufferSize;

	UE_API void					CloseWithError(const TCHAR* Error, FSocket* InSocket=nullptr);
	UE_API void					ResetStatsIfTime();
	
	/* Attempts to set the specified buffer size on our socket, will drop by 50% each time until success */
	UE_API void					SetSocketBufferSizes(FSocket* NewSocket, int32 DesiredSendSize, int32 DesiredReceiveSize);

	FThreadSafeBool			IsAttemptingConnection;
	FCriticalSection		SocketMutex;
	FSocket*				Socket;
	bool					IsListener;

	FConnectionStats		ConnectionStats;
	FConnectionStats		LastStats;
	double					TimeSinceStatsSet;
};

#undef UE_API
