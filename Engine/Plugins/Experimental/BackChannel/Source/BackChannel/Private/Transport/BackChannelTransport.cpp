// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackChannel/Transport/IBackChannelTransport.h"
#include "BackChannelCommon.h"
#include "Transport/BackChannelConnection.h"

const TCHAR* BackChannelTransport_TCP = TEXT("BackChannelTCP");

DEFINE_LOG_CATEGORY(LogBackChannel);

const int32 IBackChannelTransport::TCP=1;

class FBackChannelTransport : public IBackChannelTransport
{
public:


	virtual TSharedPtr<IBackChannelSocketConnection> CreateConnection(const int32 Type) override
	{
		check(Type == IBackChannelTransport::TCP);
		TSharedPtr<FBackChannelConnection> Connection = MakeShareable(new FBackChannelConnection());
		return Connection;	
	}
};

IMPLEMENT_MODULE(FBackChannelTransport, BackChannel)

