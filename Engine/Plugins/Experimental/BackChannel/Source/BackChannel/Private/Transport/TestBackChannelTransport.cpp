// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#if WITH_DEV_AUTOMATION_TESTS && 0

class FBackChannelTestTransport : public FAutomationTestBase
{
	enum
	{
		DefaultPort = 2020
	};
public:

	FBackChannelTestTransport(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask) {}

	void CreateListener()
	{
		if (IBackChannelTransport* BC = IBackChannelTransport::Get())
		{
			ListenerConnection = BC->CreateConnection(IBackChannelTransport::TCP);
		}
	}

	void CreateConnection1()
	{
		if (IBackChannelTransport* BC = IBackChannelTransport::Get())
		{
			ClientConnection = BC->CreateConnection(IBackChannelTransport::TCP);
		}
	}
	
	bool ConnectListenerAndConnection()
	{
		FThreadSafeBool WaitingForConnect;
		FThreadSafeBool ServerIsConnected;

		ListenerConnection->Listen(DefaultPort);
		ClientConnection->Connect(*FString::Printf(TEXT("127.0.0.1:%d"), (int)DefaultPort));

		bool ClientConnected = false;
		bool AcceptConnected = false;

		FBackChannelThreadedListener ThreadedConnection;

		ThreadedConnection.Start(ListenerConnection.ToSharedRef(), 
			FBackChannelListenerDelegate::CreateLambda([this, &AcceptConnected](TSharedPtr<IBackChannelSocketConnection> NewConnection)
		{
			AcceptConnected = true;
			AcceptedConnection = NewConnection;
			return true;
		}));

		do 
		{
			if (ClientConnected == false)
			{
				ClientConnection->WaitForConnection(0, [&ClientConnected](TSharedPtr<IBackChannelSocketConnection> NewConnection) {
					ClientConnected = true;
					return true;
				});
			}

			FPlatformProcess::SleepNoStats(0.5);

		} while (!ClientConnected || !AcceptConnected);

		return AcceptedConnection.IsValid();
	}

	TSharedPtr<IBackChannelSocketConnection>	ListenerConnection;
	TSharedPtr<IBackChannelSocketConnection>	ClientConnection;
	TSharedPtr<IBackChannelSocketConnection>	AcceptedConnection;

};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FBackChannelTestCreate, FBackChannelTestTransport, "BackChannel.TestTransport", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FBackChannelTestCreate::RunTest(const FString& Parameters)
{
	CreateListener();
	CreateConnection1();

	return ListenerConnection.IsValid() && ClientConnection.IsValid();
}


IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FBackChannelTestConnect, FBackChannelTestTransport, "Project.BackChannel.TestConnect", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FBackChannelTestConnect::RunTest(const FString& Parameters)
{
	CreateListener();
	CreateConnection1();

	ConnectListenerAndConnection();

	return ListenerConnection.IsValid() && ClientConnection.IsValid() && AcceptedConnection.IsValid();	
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FBackChannelTestSendReceive, FBackChannelTestTransport, "Project.BackChannel.TestConnect", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FBackChannelTestSendReceive::RunTest(const FString& Parameters)
{
	CreateListener();
	CreateConnection1();
	ConnectListenerAndConnection();

	FString Msg = TEXT("Hello!");
	TCHAR MsgReceived[256] = { 0 };


	int32 MsgSize = Msg.Len() * sizeof(TCHAR);

	int32 Sent = ClientConnection->SendData(*Msg, MsgSize);

	check(Sent == MsgSize);

	int32 Received = AcceptedConnection->ReceiveData(MsgReceived, 256);

	check(Received == Sent);

	check(Msg == MsgReceived);

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
