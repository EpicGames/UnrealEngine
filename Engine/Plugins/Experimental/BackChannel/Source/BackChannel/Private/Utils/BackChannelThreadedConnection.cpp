// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackChannel/Utils/BackChannelThreadedConnection.h"
#include "BackChannel/Transport/IBackChannelSocketConnection.h"
#include "HAL/RunnableThread.h"


FBackChannelThreadedListener::FBackChannelThreadedListener()
{
}

FBackChannelThreadedListener::~FBackChannelThreadedListener()
{
	Stop();
}

bool FBackChannelThreadedListener::IsRunning() const
{
	return bIsRunning;
}

void FBackChannelThreadedListener::Start(TSharedRef<IBackChannelSocketConnection> InConnection, FBackChannelListenerDelegate InDelegate)
{
	Connection = InConnection;
	Delegate = InDelegate;

	bIsRunning = true;
	bExitRequested = false;

	FRunnableThread::Create(this, TEXT("FBackChannelSocketThread"), 32 * 1024);
}

uint32 FBackChannelThreadedListener::Run()
{
	while (bExitRequested == false)
	{
		FScopeLock RunningLock(&RunningCS);

		Connection->WaitForConnection(1, [this](TSharedRef<IBackChannelSocketConnection> NewConnection) {
			return Delegate.Execute(NewConnection);
		});
	}

	bIsRunning = false;
	return 0;
}

void FBackChannelThreadedListener::Stop()
{
	bExitRequested = true;

	if (IsRunning())
	{
		FScopeLock RunLock(&RunningCS);
	}

	bExitRequested = false;
}
