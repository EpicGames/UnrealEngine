// Copyright Epic Games, Inc. All Rights Reserved.

#include "BackChannel/Utils/DispatchMap.h"
#include "BackChannel/IBackChannelPacket.h"


FBackChannelDispatchMap::FBackChannelDispatchMap()
{

}

FDelegateHandle FBackChannelDispatchMap::AddRoute(FStringView InPath, FBackChannelRouteDelegate::FDelegate Delegate)
{
	FString LowerPath = FString::ConstructFromPtrSize(InPath.GetData(), InPath.Len()).ToLower();

	if (DispatchMap.Contains(LowerPath) == false)
	{
		DispatchMap.Add(LowerPath);
	}

	return DispatchMap.FindChecked(LowerPath).Add(Delegate);
}

void FBackChannelDispatchMap::RemoveRoute(FStringView InPath, FDelegateHandle DelegateHandle)
{
	FString LowerPath = FString::ConstructFromPtrSize(InPath.GetData(), InPath.Len()).ToLower();

	auto DelegateList = DispatchMap.Find(LowerPath);

	if (DelegateList != nullptr)
	{
		DelegateList->Remove(DelegateHandle);
	}
}

/*
FBackChannelRouteDelegate& FBackChannelDispatchMap::GetAddressHandler(const TCHAR* Path)
{
	FString LowerPath = FString(Path).ToLower();

	if (DispatchMap.Contains(LowerPath) == false)
	{
		DispatchMap.Add(LowerPath);
	}

	return DispatchMap.FindChecked(LowerPath);
}
*/

bool FBackChannelDispatchMap::DispatchMessage(IBackChannelPacket& Message)
{
	FString LowerAddress = Message.GetPath().ToLower();

	bool DidDispatch = true;

	for (const auto& KV : DispatchMap)
	{
		FString LowerPath = KV.Key.ToLower();

		if (LowerAddress.StartsWith(LowerPath))
		{
			KV.Value.Broadcast(Message);
			DidDispatch = true;
		}
	}

	return DidDispatch;
}
