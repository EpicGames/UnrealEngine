// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackChannel/Types.h"

/**
 * Declares the interface for a BackChannel connection
 */
class IBackChannelConnection
{
public:

	virtual ~IBackChannelConnection() = default;

	virtual FString GetProtocolName() const = 0;

	virtual TBackChannelSharedPtr<IBackChannelPacket> CreatePacket() = 0;

	virtual int SendPacket(const TBackChannelSharedPtr<IBackChannelPacket>& Packet) = 0;

	/* Bind a delegate to a message address */
	virtual FDelegateHandle AddRouteDelegate(FStringView Path, FBackChannelRouteDelegate::FDelegate Delegate) = 0;

	/* Remove a delegate handle */
	virtual void RemoveRouteDelegate(FStringView Path, FDelegateHandle& InHandle) = 0;

	/* Set the specified send and receive buffer sizes, if supported */
	virtual void SetBufferSizes(int32 DesiredSendSize, int32 DesiredReceiveSize) = 0;
};
