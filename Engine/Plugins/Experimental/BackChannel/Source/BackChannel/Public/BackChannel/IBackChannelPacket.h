// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"

struct FBackChannelPacketType;


/**
 * Declares the interface for a BackChannel packet
 */
class IBackChannelPacket
{
public:

	virtual ~IBackChannelPacket() = default;

	virtual FBackChannelPacketType GetProtocolID() const = 0;

	virtual FString GetProtocolName() const = 0;

	virtual bool IsWritable() const = 0;

	virtual bool IsReadable() const = 0;

public:

	/* Set the path of this packet (if supported) */
	virtual int SetPath(const TCHAR* InPath) = 0;

	/* Write an int32 into the message */
	virtual int Write(const TCHAR* Name, const int32 Value) = 0;

	/* Write an int32 into message */
	virtual int Write(const TCHAR* Name, const float Value) = 0;

	/* Write a bool into the message */
	virtual int Write(const TCHAR* Name, const bool Value) = 0;

	/* Write a string into the message */
	virtual int Write(const TCHAR* Name, const TCHAR* Value) = 0;

	/* Write a string into the message */
	virtual int Write(const TCHAR* Name, const FString& Value) = 0;

	/* Write a TArray into the message */
	virtual int Write(const TCHAR* Name, const TArrayView<const uint8> Value) = 0;

	/* Write a block of data into the message */
	virtual int Write(const TCHAR* Name, const void* InBlob, int32 BlobSize) = 0;
	
public:

	/* Return the path of this packet (if supported) */
	virtual FString GetPath() const = 0;

	/* Read an int32 from the message */
	virtual int Read(const TCHAR* Name, int32& Value) = 0;

	/* Read an int32 from the message */
	virtual int Read(const TCHAR* Name, float& Value) = 0;

	/* Read a bool from the message */
	virtual int Read(const TCHAR* Name, bool& Value) = 0;

	/* Read a string from the message */
	virtual int Read(const TCHAR* Name, FString& Value) = 0;

	/* Read a TArray from the message */
	virtual int Read(const TCHAR* Name, TArray<uint8>& Data) = 0;

	/* Read a block of data from the message */
	virtual int Read(const TCHAR* InName, void* OutBlob, int32 MaxBlobSize, int32& OutBlobSize) = 0;

	/* Reset the reading position */
	virtual void ResetReading() = 0;
};
