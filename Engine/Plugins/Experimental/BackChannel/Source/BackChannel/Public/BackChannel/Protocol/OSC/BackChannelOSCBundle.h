// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackChannel/Protocol/OSC/BackChannelOSCPacket.h"


class FBackChannelOSCBundle : public FBackChannelOSCPacket
{
public:

	FBackChannelOSCBundle(OSCPacketMode InMode);
	virtual ~FBackChannelOSCBundle();

	FBackChannelOSCBundle(FBackChannelOSCBundle&& RHS);

	FBackChannelOSCBundle& operator=(FBackChannelOSCBundle&& RHS);

	virtual OSCPacketType GetType() const override { return OSCPacketType::Bundle;  }

	virtual int32 GetSize() const override;

	void AddElement(const void* InData, const int32 InSize);

	int32 GetElementCount() const;

	const TArray<uint8>& GetElement(const int32 Index) const;

	virtual TArray<uint8> WriteToBuffer() const override;

	virtual void WriteToBuffer(TArray<uint8>& Buffer) const override;

	static TSharedPtr<FBackChannelOSCBundle> CreateFromBuffer(const void* Data, int32 DataLength);

protected:

protected:

	typedef TArray<uint8> ElementData;

	OSCPacketMode			Mode;
	int64					TimeTag;
	TArray<ElementData>		Elements;

	static const ANSICHAR*		BundleHeader;
};