// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "BackChannel/IBackChannelConnection.h"
#include "BackChannel/Utils/DispatchMap.h"
#include "HAL/ThreadSafeBool.h"

#define UE_API BACKCHANNEL_API

class IBackChannelSocketConnection;
class FBackChannelOSCPacket;

/**
 *	A class that wraps an existing BackChannel connection and provides an OSC-focused interface and
 *	a background thread. Incoming messages are received on a background thread and queued until 
 *	DispatchMessages() is called. Outgoing messages are sent immediately
 */
class FBackChannelOSCConnection : FRunnable, public IBackChannelConnection
{
public:

	UE_API FBackChannelOSCConnection(TSharedRef<IBackChannelSocketConnection> InConnection);

	UE_API ~FBackChannelOSCConnection();

	/**
	 * IBackChannelConnection implementation
	 */
public:
	UE_API FString GetProtocolName() const override;

	UE_API TBackChannelSharedPtr<IBackChannelPacket> CreatePacket() override;

	UE_API int SendPacket(const TBackChannelSharedPtr<IBackChannelPacket>& Packet) override;

	/* Bind a delegate to a message address */
	UE_API FDelegateHandle AddRouteDelegate(FStringView Path, FBackChannelRouteDelegate::FDelegate Delegate) override;

	/* Remove a delegate handle */
	UE_API void RemoveRouteDelegate(FStringView Path, FDelegateHandle& InHandle) override;

	/* Sets the send and receive buffer sizes*/
	UE_API void SetBufferSizes(int32 DesiredSendSize, int32 DesiredReceiveSize) override;

public:

	UE_API bool StartReceiveThread();

	// Begin public FRunnable overrides
	UE_API virtual void Stop() override;
	// end public FRunnable overrides
	
	/* Returns our connection state as determined by the underlying BackChannel connection */
	UE_API bool IsConnected() const;

	/* Returns true if running in the background */
	UE_API bool IsThreaded() const;

    /*
        Checks for and dispatches any incoming messages. MaxTime is how long to wait if no data is ready to be read.
        This function is thread-safe and be called from a backfround thread manually or by calling StartReceiveThread()
    */
	UE_API void ReceiveAndDispatchMessages(const float MaxTime = 0);

	/* Send the provided OSC packet */
	UE_API bool SendPacket(FBackChannelOSCPacket& Packet);

	/* Set options for the specified message path */
	UE_API void SetMessageOptions(const TCHAR* Path, int32 MaxQueuedMessages);

	UE_API FString GetDescription();

protected:
	// Begin protected FRunnable overrides
	UE_API virtual uint32 Run() override;
	// End protected FRunnable overrides

	UE_API bool SendPacketData(const void* Data, const int32 DataLen);

	UE_API int32 GetMessageLimitForPath(const TCHAR* Path);

	UE_API int32 GetMessageCountForPath(const TCHAR* Path);

	UE_API void RemoveMessagesWithPath(const TCHAR* Path, const int32 Num = 0);

	UE_API void ReceiveMessages(const float MaxTime = 0);

	/* Dispatch all queued messages */
	UE_API void DispatchMessages();


protected:

	TSharedPtr<IBackChannelSocketConnection>  Connection;

	FBackChannelDispatchMap				DispatchMap;

	TArray<TSharedPtr<FBackChannelOSCPacket>> ReceivedPackets;

	TMap<FString, int32> MessageLimits;

	FThreadSafeBool		ExitRequested;
	FThreadSafeBool		IsRunning;

	FCriticalSection	ReceiveMutex;
	FCriticalSection	SendMutex;
	FCriticalSection	PacketMutex;
	TArray<uint8>		ReceiveBuffer;

	
	/* Time where we'll send a ping if no packets arrive to check the connection is alive*/
	double				PingTime = 2;

	/* Is the connection in an error state */
	bool				HasErrorState = false;

	/* How much data has been received this check? */
	int32				ReceivedDataSize = 0;

	/* How much data do we expect to receive next time? This is for OSC over TCP where the size of a packet is sent, then the packet*/
	int32				ExpectedSizeOfNextPacket = 4;
};

#undef UE_API
