// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackChannel/Protocol/OSC/BackChannelOSCPacket.h"
#include "BackChannel/IBackChannelPacket.h"
#include "BackChannel/Types.h"
#include "Misc/ByteSwap.h"

#define UE_API BACKCHANNEL_API


/**
 *	Representation of an OSC message. Data can be read/written using the explicit
 *	Read/Write functions, or the Serialize function / << operator where the behaviour
 * is overloaded based on whether the message was created for reading or writing.
 *
 *	Any failed Reads() will result in the default value of the type (e.g. 0, 0.0, false, "")
 *	being returned.
 */
class FBackChannelOSCMessage : public FBackChannelOSCPacket, public IBackChannelPacket
{
public:

	UE_API FBackChannelOSCMessage(OSCPacketMode InMode);

	UE_API FBackChannelOSCMessage(const TCHAR* Address);

	UE_API virtual ~FBackChannelOSCMessage();

	/* Move constructor */
	UE_API FBackChannelOSCMessage(FBackChannelOSCMessage&& RHS);

	/* Move operator */
	UE_API FBackChannelOSCMessage& operator=(FBackChannelOSCMessage&& RHS);

	/* Return our type */
	virtual OSCPacketType GetType() const override { return OSCPacketType::Message; }

	/* Return our size (plus any necessary padding) */
	UE_API virtual int32 GetSize() const override;

	/* Returns a buffer with the contents of this message. Data is padded per OSC requirements */
	UE_API virtual TArray<uint8> WriteToBuffer() const override;
	
	/* Writes this message into the provided buffer at an offset of Buffer.Num() */
	UE_API virtual void WriteToBuffer(TArray<uint8>& Buffer) const override;

	/* Helper to check our read/write status */
	bool IsWriting() const { return Mode == OSCPacketMode::Write; }

	/* Helper to check our read/write status */
	bool IsReading() const { return Mode == OSCPacketMode::Read; }

	/* Returns the address of this packet */
	FString GetPath() const
	{
		return Address;
	}

	/* Return our argument tags */
	const FString& GetTags() const
	{
		return TagString;
	}
	
	/* Returns the number of arguments in this message */
	int32 GetArgumentCount() const
	{
		return TagString.Len();
	}
	
	/* Returns the type of our next argument */
	TCHAR GetNextArgumentType() const
	{
		return TagString[TagIndex];
	}

	/* Return the size (plus padding) of all our arguments) */
	const int32 GetArgumentSize() const
	{
		return Buffer.Num();
	}

	/* Set our destination address */
	UE_API int SetPath(const TCHAR* Address);

	/* Reset us for reading. The next argument read will be our first argument */
	UE_API void	ResetRead();

	virtual FBackChannelPacketType GetProtocolID() const { return FBackChannelPacketType('B','O','S','C'); }

	virtual FString GetProtocolName() const { return TEXT("BackChannelOSC"); }

	virtual bool IsWritable() const { return IsWriting(); }

	virtual bool IsReadable() const { return IsReading(); }

	//! Int32 read/write

	/* Write an int32 into our arguments */
	virtual int Write(const TCHAR* InName, const int32 Value) override
	{
		check(IsWriting());
		int32 SwappedValue = !IsLegacyConnection() ? ByteSwap(Value) : Value;
		WriteTagAndData(TEXT('i'), &SwappedValue, sizeof(SwappedValue));
		return 0;
	}

	/* Write a float to our arguments */
	virtual int Write(const TCHAR* InName, const float Value) override
	{
		check(IsWriting());
		float SwappedValue = !IsLegacyConnection() ? ByteSwap(Value) : Value;
		return WriteTagAndData(TEXT('f'), &SwappedValue, sizeof(SwappedValue));
	}

	/* Write a bool to our arguments */
	virtual int Write(const TCHAR* InName, const bool Value) override
	{
		if (Value == true)
		{
			WriteTag(TEXT('T'));
		}
		else
		{
			WriteTag(TEXT('F'));
		}

		return 0;
	}

	/* Write a string to our arguments */
	virtual int Write(const TCHAR* InName, const TCHAR* Value) override
	{
		return WriteTagAndData(TEXT('s'), TCHAR_TO_ANSI(Value), FCString::Strlen(Value) + 1);
	}

	/* Write a string to our arguments */
	virtual int Write(const TCHAR* InName, const FString& Value) override
	{
		return Write(InName, *Value);
	}

	/* Write a blob of data to our arguments */
	virtual int Write(const TCHAR* InName, const void* InBlob, int32 BlobSize) override
	{
		check(IsWriting());
		if (IsLegacyConnection())
		{
			// with the legacy format it's just a fixed size
			return WriteTagAndData(TEXT('b'), InBlob, BlobSize);
		}
		else
		{
			// OSC blobs contain and int with the size before the data
			WriteTag(TEXT('b'));
			WriteData(&BlobSize, sizeof(int32));
			return WriteData(InBlob, BlobSize);
		}
	}

	/*
		Read data from the message into a TArray. It must have been serialized by the Read form for TArray(!).
		Note - data will be appended to the array.
	*/
	virtual int Write(const TCHAR* InName, const TArrayView<const uint8> InValues) override
	{
		return Write<uint8>(InName, InValues);
	}

	/* Write a TArrayView into the message */
	template<typename T>
	int Write(const TCHAR* InName, const TArrayView<const T> InValues)
	{
		if (IsLegacyConnection())
		{
			// contrary to the OSC spec, with the legacy format the size of a blob has an explicit tag for rather than simply being the first
			// four bytes of the blob data. See also Reading of arrays
			Write(TEXT("Size"), InValues.Num());
			int32 BlobSize = InValues.Num() * sizeof(T);
			return WriteTagAndData(TEXT('b'), InValues.GetData(), BlobSize);
		}
		else
		{
			return Write(InName, InValues.GetData(), InValues.Num());
		}
	}

	/* Read an int32 from our arguments */
	virtual int Read(const TCHAR* InName, int32& Value) override
	{
		check(IsReading());
		int Err = ReadTagAndData(TEXT('i'), &Value, sizeof(Value));

		if (!Err && !IsLegacyConnection())
		{
			Value = ByteSwap(Value);
		}

		return Err;
	}	

	/* Read a float from our arguments */
	virtual int Read(const TCHAR* InName, float& OutValue) override
	{
		check(IsReading());
		int Err = ReadTagAndData(TEXT('f'), &OutValue, sizeof(OutValue));

		if (!Err && !IsLegacyConnection())
		{
			OutValue = ByteSwap(OutValue);
		}

		return Err;
	}

	/* Read a bool from our arguments */
	virtual int Read(const TCHAR* InName, bool& Value) override
	{
		// try to read both tagd and see which was present
		TCHAR TrueTag = ReadTag(TEXT('T'), true);

		if (TrueTag != 0)
		{
			Value = true;
			return 0;
		}

		TCHAR FalseTag = ReadTag(TEXT('F'), true);

		if (FalseTag != 0)
		{
			Value = false;
			return 0;
		}

		return 1;
	}

	/* Read a string from our arguments.  */
	UE_API virtual int Read(const TCHAR* InName, FString& OutValue) override;

	//! Raw data blobs

	/* Read a blob of data from our arguments */
	UE_API virtual int Read(const TCHAR* InName, void* InBlob, int32 MaxBlobSize, int32& OutBlobSize) override;	

	/* Reset the reading position */
	virtual void ResetReading() override
	{
		ResetRead();
	}

	/* 
		Read data from the message into a TArray. It must have been serialized by the Read form for TArray(!). 
		Note - data will be appended to the array.
	*/
	virtual int Read(const TCHAR* InName, TArray<uint8>& DataArray) override
	{
		return Read<uint8>(InName, DataArray);
	}

	/*
	 *	Read a TArray of type T from our arguments. This is a helper that reads an int
	*	for the size, then allocated and reads a blob of sizeof(t) * NumItems
	 */
	template<typename T>
	int Read(const TCHAR* InName, TArray<T>& DataArray)
	{
		int32 BlobSize = 0;
		int32 NumNewElements = 0;

		if (IsLegacyConnection())
		{
			// contrary to the OSC spec, with the legacy format the size of a blob has an explicit tag for rather than simply being the first
			// four bytes of the blob data. See also reading of arrays
			Read(TEXT("Size"), NumNewElements);
			BlobSize = NumNewElements * sizeof(T);

			// Add enough space
			int NumCurrentElements = DataArray.Num();
			DataArray.AddUninitialized(NumNewElements);

			// read the tag and data directly, don't use the blob read
			return ReadTagAndData(TEXT('b'), DataArray.GetData() + NumCurrentElements, BlobSize);
		}
		else
		{
			// Read the size of the blob and covert to element count
			Read(InName, nullptr, 0, BlobSize);
			NumNewElements = BlobSize / sizeof(T);

			// Add enough space
			int NumCurrentElements = DataArray.Num();
			DataArray.AddUninitialized(NumNewElements);

			return Read(InName, DataArray.GetData() + NumCurrentElements, BlobSize, BlobSize);
		}
	}
	

	/* Serialize helper that will read/write based on the open mode of this message */
	template<typename T>
	void Serialize(const TCHAR* Name, T& Value)
	{
		if (IsWriting())
		{
			Write(Name, Value);
		}
		else
		{
			Read(Name, Value);
		}
	}

	/* Serialize helper that will read/write based on the open mode of this message */
	void Serialize(const TCHAR* Name, void* InBlob, int32 BlobSize)
	{
		if (IsReading())
		{
			int32 OutSize(0);
			Read(Name, InBlob, BlobSize, OutSize);
		}
		else
		{
			Write(Name, InBlob, BlobSize);
		}
	}

	static int32 RoundedArgumentSize(int32 ArgSize)
	{
		return ((ArgSize + 3) / 4) * 4;
	}

	static UE_API TSharedPtr<FBackChannelOSCMessage> CreateFromBuffer(const void* Data, int32 DataLength);

	static void SetLegacyMode(const bool bEnable) { bIsLegacyConnection  = bEnable; }
	static bool IsLegacyConnection() { return bIsLegacyConnection; }

protected:

	/* Return our sizes for each item (plus any necessary padding) */
	UE_API void GetComponentSizes(int32& OutAddressSize, int32& OutTagSize, int32& OutBufferSize) const;

	UE_API int Serialize(const TCHAR Code, void* InData, int32 InSize);

	UE_API TCHAR ReadTag(const TCHAR ExpectedTag, bool SuppressWarning=false);
	UE_API int ReadData(void* InData, int32 InSize);
	UE_API int ReadTagAndData(const TCHAR ExpectedTag, void* InData, int32 InSize);

	UE_API int WriteTag(const TCHAR Code);
	UE_API int WriteTagAndData(const TCHAR Code, const void* InData, int32 InSize);
	UE_API int WriteData(const void* InData, int32 InSize);

protected:

	OSCPacketMode		Mode;
	FString				Address;
	FString				TagString;
	int					TagIndex;
	int					BufferIndex;
	TArray<uint8>		Buffer;

	static UE_API bool bIsLegacyConnection;
};

/*
BACKCHANNEL_API FBackChannelOSCMessage& operator << (FBackChannelOSCMessage& Msg, int32& Value);

BACKCHANNEL_API FBackChannelOSCMessage& operator << (FBackChannelOSCMessage& Msg, float& Value);

BACKCHANNEL_API FBackChannelOSCMessage& operator << (FBackChannelOSCMessage& Msg, bool& Value);

BACKCHANNEL_API FBackChannelOSCMessage& operator << (FBackChannelOSCMessage& Msg, TCHAR& Value);

BACKCHANNEL_API FBackChannelOSCMessage& operator << (FBackChannelOSCMessage& Msg, FString& Value);

template <typename T>
FBackChannelOSCMessage& operator << (FBackChannelOSCMessage& Msg, TArray<T>& Value)
{
	if (Msg.IsWriting())
	{
		Msg.Write(nullptr, Value);
	}
	else
	{
		Msg.Read(nullptr, Value);
	}

	return Msg;
}


template <typename T>
FBackChannelOSCMessage& SerializeOut(FBackChannelOSCMessage& Msg, const T& Value)
{
	T Tmp = Value;
	Msg << Tmp;
	return Msg;
}
*/

#undef UE_API
