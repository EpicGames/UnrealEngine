// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API BACKCHANNEL_API

enum class OSCPacketMode
{
	Read,
	Write
};

enum class OSCPacketType
{
	Invalid,
	Message,
	Bundle
};

/**
 *	Base class for OSC messages
 */
class FBackChannelOSCPacket
{
public:

	virtual ~FBackChannelOSCPacket() = default;

	/* Return the total size in bytes of this message */
	virtual int32	GetSize() const = 0;

	/* Return the type of this packet */
	virtual OSCPacketType GetType() const = 0;
	
	/* Return a buffer with a copy of the contents of this packet */
	virtual TArray<uint8> WriteToBuffer() const =0;

	/* Write the contents of this packet into the specified buffer starting at an offset of Buffer.Num()  */
	virtual void WriteToBuffer(TArray<uint8>& Buffer) const = 0;

	/* Static helper to determine the type of packet (if any) at the specified address */
	static UE_API OSCPacketType GetType(const void* Data, int32 DataLength);

	/* Helper to construct a packet (and in the case of a bundle, any sub-packets) from the specified buffer */
	static UE_API TSharedPtr<FBackChannelOSCPacket> CreateFromBuffer(const void* Data, int32 DataLength);
};

class FBackChannelOSCNullPacket : public FBackChannelOSCPacket
{
public:
	virtual int32 GetSize() const override { return 0; }

	virtual OSCPacketType GetType() const override { return OSCPacketType::Invalid;  }
};

#undef UE_API
