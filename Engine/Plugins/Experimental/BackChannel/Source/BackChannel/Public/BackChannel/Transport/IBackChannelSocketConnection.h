// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformTime.h"
#include "Templates/SharedPointer.h"

class FSocket;

// todo (agrant 2017/12/29): concept of 'connection' should be a base class with persistent connection subclass?

/**
 *	Base class that describes a back-channel connection. The underlying behavior will depend on the type
 *	of connection that was requested from the factory
 */
class IBackChannelSocketConnection
{
public:

	struct FConnectionStats
	{
		// lifetime stats
		int		PacketsSent = 0;
		int		PacketsReceived = 0;
		int		BytesSent = 0;
		int		BytesReceived = 0;
		int		Errors = 0;

		// time these events occurred
		double		LastSendTime = 0;
		double		LastReceiveTime = 0;
		double		LastErrorTime = FPlatformTime::Seconds();
		double		LastSuccessTime = FPlatformTime::Seconds();
	};

	
	// todo (agrant 2017/12/29): Should remove 'Connect' and instead return a connected (or null..) socket
	// from the factory

	/* Start connecting to the specified port for incoming connections. Use WaitForConnection to check status. */
	virtual bool Connect(const TCHAR* InEndPoint) = 0;

	/* Start listening on the specified port for incoming connections. Use WaitForConnection to accept one. */
	virtual bool Listen(const int16 Port) = 0;

	/* Close the connection */
	virtual void Close() = 0;

	/* Waits for an icoming or outgoing connection to be made */
	virtual bool WaitForConnection(double InTimeout, TFunction<bool(TSharedRef<IBackChannelSocketConnection>)> InDelegate) = 0;

	/* Returns true if this connection is currently listening for incoming connections */
	virtual bool IsListening() const = 0;

	/* Returns true if this connection is connected to another */
	virtual bool IsConnected() const = 0;

	/* Send data via our connection */
	virtual int32 SendData(const void* InData, const int32 InSize) = 0;
	
	/* Receive data from our connection. The returned value is the number of bytes read, to a max of BufferSize */
	virtual int32 ReceiveData(void* OutBuffer, const int32 BufferSize) = 0;

	/* Return the underlying socket (if any) for this connection */
	virtual FString GetDescription() const = 0;

	/* Return the underlying socket (if any) for this connection */
	virtual FSocket* GetSocket() = 0;

	/* Todo - Proper stats */
	virtual uint32	GetPacketsReceived() const = 0;

	/* Set the specified send and receive buffer sizes, if supported */
	virtual void SetBufferSizes(int32 DesiredSendSize, int32 DesiredReceiveSize) = 0;

	/* Return stats about this connection */
	virtual const FConnectionStats& GetConnectionStats() const = 0;

protected:

	IBackChannelSocketConnection() {}
	virtual ~IBackChannelSocketConnection() = default;
};
