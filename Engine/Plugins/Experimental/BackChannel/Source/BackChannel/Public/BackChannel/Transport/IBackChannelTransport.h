// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#define UE_API BACKCHANNEL_API

class IBackChannelSocketConnection;

/**
 *	Main module and factory interface for Backchannel connections
 */
class IBackChannelTransport : public IModuleInterface
{
public:

	static inline bool IsAvailable(void)
	{
		return Get() != nullptr;
	}

	static inline IBackChannelTransport* Get(void)
	{
		return FModuleManager::LoadModulePtr<IBackChannelTransport>("BackChannel");
	}

	virtual TSharedPtr<IBackChannelSocketConnection> CreateConnection(const int32 Type) = 0;

public:

	static UE_API const int TCP;

protected:

	IBackChannelTransport() {}
	virtual ~IBackChannelTransport() = default;
};

#undef UE_API
