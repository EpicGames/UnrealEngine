// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class IBackChannelConnection;
class IBackChannelPacket;

struct FBackChannelPacketType
{
	FBackChannelPacketType(char A, char B, char C, char D)
		: ID{ A,B,C,D }
	{
		FMemory::Memzero(ID);
	}
	char ID[4];
};


DECLARE_MULTICAST_DELEGATE_OneParam(FBackChannelRouteDelegate, IBackChannelPacket&)

template <class ObjectType>
using TBackChannelSharedPtr = TSharedPtr<ObjectType, ESPMode::ThreadSafe>;
