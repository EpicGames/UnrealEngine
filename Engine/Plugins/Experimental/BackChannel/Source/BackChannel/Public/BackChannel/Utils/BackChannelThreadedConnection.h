// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Runnable.h"
#include "HAL/ThreadSafeBool.h"

#define UE_API BACKCHANNEL_API

class IBackChannelSocketConnection;

class FSocket;

DECLARE_DELEGATE_RetVal_OneParam(bool, FBackChannelListenerDelegate, TSharedRef<IBackChannelSocketConnection>);

/**
 * BackChannelClient implementation.
 */
class FBackChannelThreadedListener : public FRunnable, public TSharedFromThis<FBackChannelThreadedListener>
{
public:

	UE_API FBackChannelThreadedListener();
	UE_API ~FBackChannelThreadedListener();

	UE_API void Start(TSharedRef<IBackChannelSocketConnection> InConnection, FBackChannelListenerDelegate InDelegate);

	UE_API virtual void Stop() override;

	UE_API bool IsRunning() const;

protected:

	UE_API virtual uint32 Run() override;

private:

	TSharedPtr<IBackChannelSocketConnection>		Connection;
	FBackChannelListenerDelegate			Delegate;
	
	FThreadSafeBool							bExitRequested;
	FThreadSafeBool							bIsRunning;
	FCriticalSection						RunningCS;
};

#undef UE_API
