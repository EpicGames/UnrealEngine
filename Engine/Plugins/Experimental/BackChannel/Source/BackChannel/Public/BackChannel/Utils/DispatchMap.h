// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BackChannel/Types.h"

#define UE_API BACKCHANNEL_API



class FBackChannelDispatchMap
{

public:

	UE_API FBackChannelDispatchMap();

	virtual ~FBackChannelDispatchMap() = default;

	UE_API FDelegateHandle AddRoute(FStringView Path, FBackChannelRouteDelegate::FDelegate Delegate);
	
	UE_API void RemoveRoute(FStringView Path, FDelegateHandle DelegateHandle);

	UE_API bool	DispatchMessage(IBackChannelPacket& Message);

protected:

	TMap<FString, FBackChannelRouteDelegate> DispatchMap;

};

#undef UE_API
